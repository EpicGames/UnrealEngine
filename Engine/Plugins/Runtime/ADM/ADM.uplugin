{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Audio Definition Model (ADM)",
	"Description": "Currently only supports output spatialized using WASAPI aggregate output channels and spatial ADM information transmitted using Open Sound Control (OSC).",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ADMSpatialization",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OSC",
			"Enabled": true
		}
	]
}