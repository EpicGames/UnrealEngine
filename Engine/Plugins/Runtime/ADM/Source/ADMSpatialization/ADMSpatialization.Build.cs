// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ADMSpatialization : ModuleRules
	{
		public ADMSpatialization(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioExtensions",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"Networking",
					"OSC"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioMixerCore",
					"AudioMixer",
					"SignalProcessing"
				}
			);
		}
	}
}
