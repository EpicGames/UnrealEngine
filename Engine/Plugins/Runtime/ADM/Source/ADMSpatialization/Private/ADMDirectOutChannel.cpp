// Copyright Epic Games, Inc. All Rights Reserved.

#include "ADMDirectOutChannel.h"

#include "Audio.h"
#include "AudioMixer.h"
#include "DSP/FloatArrayMath.h"
#include "Logging/LogMacros.h"


namespace UE::ADM::Spatialization
{
	TUniquePtr<Audio::FAlignedFloatBuffer> FSourceDirectOut::SilienceBuffer;

	FSourceDirectOut::FSourceDirectOut(const int32 InChannelIndex, const uint32 InNumSamples, Audio::IAudioMixerPlatformInterface* InMixerPlatform)
	{
		ChannelIndex = InChannelIndex;
		MixerPlatform = InMixerPlatform;
		InitializeTempBuffers(InNumSamples);
	}

	FSourceDirectOut::~FSourceDirectOut()
	{
		FScopeLock ScopeLock(&DestructorCriticalSection);

		MixerPlatform = nullptr;
	}

	void FSourceDirectOut::InitializeTempBuffers(const uint32 InNumSamples)
	{
		if (!SilienceBuffer.IsValid())
		{
			SilienceBuffer = MakeUnique<Audio::FAlignedFloatBuffer>();

			if (SilienceBuffer.IsValid())
			{
				SilienceBuffer->SetNumZeroed(InNumSamples);
			}
		}
	}

	void FSourceDirectOut::ProcessDirectOut(const FAudioPluginSourceInputData& InInputData)
	{
		if (!DestructorCriticalSection.TryLock())
		{
			return;
		}

		if (MixerPlatform)
		{
			if (InInputData.NumChannels == 1)
			{
				// Output is typically clamped by the mixer after all the sources have been mixed into the output buffer.
				// Since we are sending each source to a direct output, clamp here.
				Audio::FAlignedFloatBuffer* InputBuffer = const_cast<Audio::FAlignedFloatBuffer*>(InInputData.AudioBuffer);
				Audio::ArrayRangeClamp(*InputBuffer, -1.0f, 1.0f);

				MixerPlatform->SubmitDirectOutBuffer(ChannelIndex, *InInputData.AudioBuffer);
			}
		}

		DestructorCriticalSection.Unlock();
	}

	void FSourceDirectOut::ProcessSilence()
	{
		if (!DestructorCriticalSection.TryLock())
		{
			return;
		}

		if (MixerPlatform)
		{
			MixerPlatform->SubmitDirectOutBuffer(ChannelIndex, *SilienceBuffer);
		}

		DestructorCriticalSection.Unlock();
	}

} // namespace UE::ADM::Spatialization
