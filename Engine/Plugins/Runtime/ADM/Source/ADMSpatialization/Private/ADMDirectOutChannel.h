// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "Containers/Array.h"
#include "DSP/AlignedBuffer.h"
#include "IAudioExtensionPlugin.h"


namespace UE::ADM::Spatialization
{
	class ADMSPATIALIZATION_API FSourceDirectOut
	{
	public:
		FSourceDirectOut() = delete;
		FSourceDirectOut(const int32 InChannelIndex, const uint32 InNumSamples, Audio::IAudioMixerPlatformInterface* InMixerPlatform);
		~FSourceDirectOut();

		void ProcessDirectOut(const FAudioPluginSourceInputData& InInputData);
		void ProcessSilence();

		void SetIsActive(const bool bInIsActive) { bIsActive = bInIsActive;  }
		bool GetIsActive() const { return bIsActive; }

		void SetSourceId(const int32 InSourceId) { SourceId = InSourceId; }
		int32 GetSourceId() const { return SourceId; }

	private:
		static TUniquePtr<Audio::FAlignedFloatBuffer> SilienceBuffer;

		FCriticalSection DestructorCriticalSection;

		bool bIsActive = false;
		Audio::IAudioMixerPlatformInterface* MixerPlatform = nullptr;
		int32 SourceId = INDEX_NONE;
		int32 ChannelIndex = INDEX_NONE;

		void InitializeTempBuffers(const uint32 InNumSamples);
	};

} // namespace UE::ADM::Spatialization
