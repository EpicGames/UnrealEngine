// Copyright Epic Games, Inc. All Rights Reserved.

#include "ADMSpatialization.h"

#include "ADMDirectOutChannel.h"
#include "ADMSpatializationLog.h"
#include "ADMSpatializationModule.h"
#include "AudioMixerDevice.h"
#include "Math/Transform.h"
#include "OSCClient.h"
#include "OSCMessage.h"
#include "OSCTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ADMSpatialization)

void UADMEngineSubsystem::SetSendIPAddress(const FString& InIPAddress, int32 Port)
{
	using namespace UE::ADM::Spatialization;

	const FString EndpointStr = FString::Printf(TEXT("%s:%i"), *InIPAddress, Port);
	FIPv4Endpoint Endpoint;
	if (FIPv4Endpoint::Parse(EndpointStr, Endpoint))
	{
		FModule& Module = FModuleManager::Get().LoadModuleChecked<FModule>("ADMSpatialization");
		FADMSpatializationFactory& SpatFactory = Module.GetFactory();
		SpatFactory.SetSendIPEndpoint(Endpoint);
	}
	else
	{
		UE_LOGF(LogADMSpatialization, Error, "Failed to parse specified ADM Spatialization client endpoint '%ls'. Client IP not updated for ADM Spatialization.", *EndpointStr);
	}
}

namespace UE::ADM::Spatialization
{
	namespace SpatPrivate
	{
		FString SendEndpointCVar;
		FAutoConsoleVariableRef CVarOSCADMSendEndpoint(
			TEXT("au.ADM.Spatialization.OSCSendEndpoint"),
			SendEndpointCVar,
			TEXT("Override send (client) endpoint for ADM spatializer messaging (in the form 127.0.0.1:8000).")
			TEXT("Default: Empty (Does not override project setting)"),
			ECVF_Default);

		int32 PositionAddressOffsetCVar = -1;
		FAutoConsoleVariableRef CVarOSCADMPositionAddressOffset(
			TEXT("au.ADM.Spatialization.OSCPositionAddressOffset"),
			PositionAddressOffsetCVar,
			TEXT("Applies an index offset to all object Ids translated to OSC position source addresses.")
			TEXT("Default: -1 (Use system default offset)"),
			ECVF_Default);

		static const FString SystemName(TEXT("ADM Spatialization"));

	} // namespace SpatPrivate

	FADMClient::FADMClient(const FIPv4Endpoint& InEndpoint, const int32 InObjectIndexOffset) :
		ObjectIndexOffset(InObjectIndexOffset)
	{
		FIPv4Endpoint IPEndpoint = InEndpoint;
		if (!SpatPrivate::SendEndpointCVar.IsEmpty())
		{
			FIPv4Endpoint::Parse(SpatPrivate::SendEndpointCVar, IPEndpoint);
		}

		ClientProxy = UE::OSC::IClientProxy::Create(SpatPrivate::SystemName);
		ClientProxy->SetSendIPEndpoint(IPEndpoint);
	}

	int32 FADMClient::GetObjectIndexOffset() const
	{
		if (SpatPrivate::PositionAddressOffsetCVar >= 0)
		{
			return SpatPrivate::PositionAddressOffsetCVar;
		}
		else
		{
			return ObjectIndexOffset;
		}
	}

	FOSCAddress FADMClient::CreateConfigAddress(int32 InObjIndex, FString InMethod) const
	{
		TArray<FString> Containers
		{
			TEXT("adm"),
			TEXT("obj"),
			FString::FromInt(InObjIndex + GetObjectIndexOffset()),
			TEXT("config")
		};

		FOSCAddress OSCAddress;
		OSCAddress.Set(MoveTemp(Containers), MoveTemp(InMethod));
		return OSCAddress;
	}

	FOSCAddress FADMClient::CreatePositionAddress(int32 InObjIndex, FString InMethod) const
	{
		TArray<FString> Containers
		{
			TEXT("adm"),
			TEXT("obj"),
			FString::FromInt(InObjIndex + GetObjectIndexOffset())
		};

		FOSCAddress OSCAddress;
		OSCAddress.Set(MoveTemp(Containers), MoveTemp(InMethod));
		return OSCAddress;
	}

	void FADMClient::InitObjectIndex(int32 InObjIndex, bool bCartesian)
	{
		using namespace UE::OSC;

		if (ClientProxy.IsValid())
		{
			FOSCAddress OSCAddress = CreateConfigAddress(InObjIndex, TEXT("cartesian"));
			FOSCMessage Msg(MoveTemp(OSCAddress), { FOSCData((int32)(bCartesian)) });
			ClientProxy->SendMessage(Msg);

			UE_LOGF(LogADMSpatialization, Verbose, "InitObjectIndex: %d", InObjIndex + GetObjectIndexOffset());
		}
	}

	void FADMClient::SetPosition(int32 InObjIndex, const FVector& InPosition)
	{
		using namespace UE::OSC;

		if (ClientProxy.IsValid())
		{
			FVector ADMPosition = UnrealToADMCoordinates(InPosition);

			FOSCAddress OSCAddress = CreatePositionAddress(InObjIndex, TEXT("xyz"));
			FOSCMessage Msg(MoveTemp(OSCAddress), { FOSCData((float)ADMPosition.X) , FOSCData((float)ADMPosition.Y), FOSCData((float)ADMPosition.Z) });

			ClientProxy->SendMessage(Msg);
		}
	}

	bool FADMClient::IsSet() const
	{
		return ClientProxy.IsValid();
	}

	FString FADMSpatializationFactory::GetDisplayName()
	{
		return SpatPrivate::SystemName;
	}

	bool FADMSpatializationFactory::SupportsPlatform(const FString& PlatformName)
	{
		return true;
	}

	TAudioSpatializationPtr FADMSpatializationFactory::CreateNewSpatializationPlugin(FAudioDevice* OwningDevice)
	{
		return MakeShared<FADMSpatialization>();
	}

	UClass* FADMSpatializationFactory::GetCustomSpatializationSettingsClass() const
	{
		return UADMSpatializationSourceSettings::StaticClass();
	};

	bool FADMSpatializationFactory::IsExternalSend()
	{
		// Because this plugin sends all spatialized sources to direct outputs,
		// indicate to the engine that it acts as an external send.
		return true;
	}

	int32 FADMSpatializationFactory::GetMaxSupportedChannels()
	{
		return 1;
	}

	void FADMSpatialization::Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
		FModule& Module = FModuleManager::Get().LoadModuleChecked<FModule>("ADMSpatialization");
		FADMSpatializationFactory& SpatFactory = Module.GetFactory();
		const FIPv4Endpoint Endpoint = SpatFactory.GetSendIPEndpoint();

		int32 NumDirectOutChannels = 0;
		Audio::FMixerDevice* AudioMixerDevice = static_cast<Audio::FMixerDevice*>(InitializationParams.AudioDevicePtr);
		if (!ensureMsgf(AudioMixerDevice, TEXT("Unable to initialize FADMSpatialization, null audio device")))
		{
			return;
		}

		Audio::IAudioMixerPlatformInterface* MixerPlatform = AudioMixerDevice->GetAudioMixerPlatform();
		if (!ensureMsgf(MixerPlatform, TEXT("Unable to initialize FADMSpatialization, null mixer platform")))
		{
			return;
		}

		NumBedChannels = AudioMixerDevice->GetNumDeviceChannels();
		NumDirectOutChannels = AudioMixerDevice->GetNumDirectOutChannels();
		
		NumSources = InitializationParams.NumSources;
		SampleRate = InitializationParams.SampleRate;

		checkf(NumSources >= 0 && NumSources <= AudioMixerDevice->GetMaxSources(), TEXT("NumSources is expected to be in the range of 0 to max sources for the mixer device"));
		checkf(SampleRate == AudioMixerDevice->GetSampleRate(), TEXT("SampleRate is expected to match the mixer device sample rate"));

		DirectOuts.Reset(InitializationParams.NumSources);
		for (uint32 Index = 0; Index < InitializationParams.NumSources; Index++)
		{
			DirectOuts.Emplace(Index, InitializationParams.BufferLength, MixerPlatform);
		}

		SourceIdChannelMap.Empty(InitializationParams.NumSources);

		SetClient(FADMClient(Endpoint, NumBedChannels));
	}

	void FADMSpatialization::Shutdown()
	{
		DirectOuts.Reset();
		SourceIdChannelMap.Reset();
	}

	void FADMSpatialization::SetClient(FADMClient&& InClient)
	{
		Client = MoveTemp(InClient);
	}

	bool FADMSpatialization::IsSpatializationEffectInitialized() const
	{
		return true;
	}

	void FADMSpatialization::OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, USpatializationPluginSourceSettingsBase* InSettings)
	{
		checkf((int32)SourceId < NumSources, TEXT("SourceId is expected to be less than the initialized max number of sources"));

		if (ensure(!SourceIdChannelMap.Contains(SourceId)))
		{
			int32 ChannelIndex = INDEX_NONE;
			for (int32 Index = 0; Index < DirectOuts.Num(); ++Index)
			{
				if (!DirectOuts[Index].GetIsActive())
				{
					ChannelIndex = Index;
					break;
				}
			}

			if (DirectOuts.IsValidIndex(ChannelIndex))
			{
				DirectOuts[ChannelIndex].SetIsActive(true);
				DirectOuts[ChannelIndex].SetSourceId(SourceId);

				SourceIdChannelMap.Emplace(SourceId, ChannelIndex);

				// The client uses the channel index as the object index per
				// the ADM OSC spec
				Client.InitObjectIndex(ChannelIndex);

				UE_LOGF(LogADMSpatialization, Verbose, "OnInitSource: %d [%d]", SourceId, ChannelIndex);
			}
			else
			{
				UE_LOGF(LogADMSpatialization, Warning, "Failed to find available direct out channel for SourceId: %d", SourceId);
			}
		}
	}

	void FADMSpatialization::OnReleaseSource(const uint32 SourceId)
	{
		checkf((int32)SourceId < NumSources, TEXT("SourceId is expected to be less than the initialized max number of sources"));

		if (ensure(SourceIdChannelMap.Contains(SourceId)))
		{
			const int32 ChannelIndex = SourceIdChannelMap[SourceId];

			if (ensure(DirectOuts.IsValidIndex(ChannelIndex)))
			{
				DirectOuts[ChannelIndex].SetIsActive(false);
				DirectOuts[ChannelIndex].SetSourceId(INDEX_NONE);
			}

			SourceIdChannelMap.Remove(SourceId);
		}
	}

	void FADMSpatialization::ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
	{
		checkf(InputData.SourceId < NumSources, TEXT("SourceId is expected to be less than the initialized max number of sources"));

		if (SourceIdChannelMap.Contains(InputData.SourceId))
		{
			const int32 ChannelIndex = SourceIdChannelMap[InputData.SourceId];

			if (DirectOuts.IsValidIndex(ChannelIndex) && DirectOuts[ChannelIndex].GetIsActive())
			{
				DirectOuts[ChannelIndex].ProcessDirectOut(InputData);

				FTransform ListenerTransform = FTransform(InputData.SpatializationParams->ListenerPosition);
				FVector ListenerRelEmitterPos = ListenerTransform.Inverse().TransformPosition(InputData.SpatializationParams->EmitterWorldPosition);

				Client.SetPosition(ChannelIndex, ListenerRelEmitterPos.GetSafeNormal());
			}
		}
	}

	void FADMSpatialization::OnAllSourcesProcessed()
	{
		for (FSourceDirectOut& DirectOut : DirectOuts)
		{
			if (!DirectOut.GetIsActive())
			{
				// Send silience out non-active outputs to retain time alignment with other outputs
				DirectOut.ProcessSilence();
			}
		}
	}

} // namespace UE::ADMSpatialization
