// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ADMDirectOutChannel.h"
#include "AudioMixer.h"
#include "DSP/AlignedBuffer.h"
#include "HAL/CriticalSection.h"
#include "IAudioExtensionPlugin.h"
#include "OSCClient.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "Templates/UniquePtr.h"

#include "ADMSpatialization.generated.h"


// Forward Declarations
class IOSCClientProxy;


UCLASS()
class ADMSPATIALIZATION_API UADMSpatializationSourceSettings : public USpatializationPluginSourceSettingsBase
{
	GENERATED_BODY()

public:
	UADMSpatializationSourceSettings() = default;
};

UCLASS()
class ADMSPATIALIZATION_API UADMEngineSubsystem : public UAudioEngineSubsystem
{
	GENERATED_BODY()

	//~ Begin USubsystem interface
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override { return true; }
	virtual void Initialize(FSubsystemCollectionBase& Collection) override { }
	virtual void Deinitialize() override { }
	//~ End USubsystem interface

	// Set the IP Address to communicate ADM updates over OSC to
	UFUNCTION(BlueprintCallable, Category = "ADM|OSC", meta = (DisplayName = "Set Send IP Address", Keywords = "osc adm message"))
	void SetSendIPAddress(const FString& IPAddress, int32 Port);
};


namespace UE::ADM::Spatialization
{
	class ADMSPATIALIZATION_API FADMSpatializationFactory : public IAudioSpatializationFactory
	{
	public:
		FADMSpatializationFactory() = default;
		~FADMSpatializationFactory() = default;

		virtual FString GetDisplayName() override;
		virtual bool SupportsPlatform(const FString& PlatformName) override;
		virtual TAudioSpatializationPtr CreateNewSpatializationPlugin(FAudioDevice* OwningDevice) override;
		virtual UClass* GetCustomSpatializationSettingsClass() const override;
		virtual bool IsExternalSend() override;
		virtual int32 GetMaxSupportedChannels() override;

		void SetSendIPEndpoint(const FIPv4Endpoint& InIPEndpoint)
		{
			FScopeLock Lock(&ClientIPCritSec);
			IPEndpoint = InIPEndpoint;
		}

		const FIPv4Endpoint& GetSendIPEndpoint() const
		{
			FScopeLock Lock(&ClientIPCritSec);
			return IPEndpoint;
		}

	private:
		mutable FCriticalSection ClientIPCritSec;

		FIPv4Endpoint IPEndpoint;
	};


	class ADMSPATIALIZATION_API FADMClient
	{
	public:
		FADMClient() = default;
		FADMClient(const FIPv4Endpoint& InEndpoint, const int32 InObjectIndexOffset = 0);

		void InitObjectIndex(int32 InObjIndex, bool bCartesian = true);
		bool IsSet() const;

		void SetPosition(int32 InObjIndex, const FVector& InPosition);

	private:
		TUniquePtr<UE::OSC::IClientProxy> ClientProxy;
		int32 ObjectIndexOffset = 0;

		int32 GetObjectIndexOffset() const;

		FOSCAddress CreateConfigAddress(int32 InObjIndex, FString InMethod) const;
		FOSCAddress CreatePositionAddress(int32 InObjIndex, FString InMethod) const;

		FORCEINLINE FVector UnrealToADMCoordinates(const FVector& InPosition)
		{
			/*       
				UNREAL                    ADM-OSC
				 Z                          Z
				 |    X                     |    Y
				 |   /                      |   /
				 |  /                       |  /
				 | /                        | /
				 |/_______________Y         |/_______________X
			*/

			return { InPosition.Y, InPosition.X, InPosition.Z };
		}
	};


	class ADMSPATIALIZATION_API FADMSpatialization : public IAudioSpatialization
	{
	public:
		FADMSpatialization() = default;
		~FADMSpatialization() = default;

		virtual void Initialize(const FAudioPluginInitializationParams InitializationParams) override;
		virtual bool IsSpatializationEffectInitialized() const override;
		virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, USpatializationPluginSourceSettingsBase* InSettings) override;
		virtual void OnReleaseSource(const uint32 SourceId) override;
		virtual void ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData) override;
		virtual void OnAllSourcesProcessed() override;
		virtual void Shutdown () override;

		void SetClient(FADMClient&& InClient);

	private:
		int32 NumBedChannels = 0;
		int32 NumSources = 0;
		float SampleRate = 0.f;

		TArray<FSourceDirectOut> DirectOuts;
		/** Maps Source ID to Channel index */
		TMap<int32, int32> SourceIdChannelMap;

		FADMClient Client;
	};
}  // namespace UE::ADM::Spatialization
