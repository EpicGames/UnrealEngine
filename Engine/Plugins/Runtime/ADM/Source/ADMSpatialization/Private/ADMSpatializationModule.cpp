// Copyright Epic Games, Inc. All Rights Reserved.

#include "ADMSpatializationModule.h"

#include "ADMSpatializationSettings.h"
#include "ADMSpatializationLog.h"

DEFINE_LOG_CATEGORY(LogADMSpatialization);


namespace UE::ADM::Spatialization
{
	void FModule::StartupModule()
	{
		IModularFeatures::Get().RegisterModularFeature(IAudioSpatializationFactory::GetModularFeatureName(), &SpatializationFactory);

		if (const UADMSpatializationSettings* Settings = GetDefault<UADMSpatializationSettings>())
		{
			FIPv4Address IPAddress;
			if (FIPv4Address::Parse(Settings->IPAddress, IPAddress))
			{
				SpatializationFactory.SetSendIPEndpoint(FIPv4Endpoint(IPAddress, Settings->IPPort));
			}
			else
			{
				UE_LOGF(LogADMSpatialization, Display, "Failed to parse specified default ADM Spatialization client endpoint in developer settings. Default IP not set for ADM Spatialization.");
			}
		}
		else
		{
			UE_LOGF(LogADMSpatialization, Error, "Failed to find ADM Spatialization Developer Settings. Default IP not set for ADM Spatialization.");
		}

		// Enable aggregate devices so we can support large channel counts
		constexpr bool bEnable = true;
		FAudioDeviceManager::EnableAggregateDeviceSupport(bEnable);
	}

	void FModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(IAudioSpatializationFactory::GetModularFeatureName(), &SpatializationFactory);
	};

	FADMSpatializationFactory& FModule::GetFactory()
	{
		return SpatializationFactory;
	}
} // namespace UE::ADM::Spatialization

IMPLEMENT_MODULE(UE::ADM::Spatialization::FModule, ADMSpatialization)
