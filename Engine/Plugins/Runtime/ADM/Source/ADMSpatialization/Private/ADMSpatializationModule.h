// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ADMSpatialization.h"
#include "Modules/ModuleInterface.h"


namespace UE::ADM::Spatialization
{
	class ADMSPATIALIZATION_API FModule : public IModuleInterface
	{
	public:
		FADMSpatializationFactory& GetFactory();

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:
		FADMSpatializationFactory SpatializationFactory;
	};
} // namespace UE::ADM::Spatialization
