// Copyright Epic Games, Inc. All Rights Reserved.

#include "ADMSpatializationSettings.h"

#include "ADMSpatialization.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "ADMSpatializationModule.h"


#if WITH_EDITOR
void UADMSpatializationSettings::ADMConnect()
{
	using namespace UE::ADM::Spatialization;

	FIPv4Address ADMAddress;
	FIPv4Address::Parse(IPAddress, ADMAddress);
	FIPv4Endpoint ADMEndpoint(ADMAddress, IPPort);

	FModule& ADMSpatializationModule = FModuleManager::Get().LoadModuleChecked<FModule>("ADMSpatialization");
	FADMSpatializationFactory& SpatFactory = ADMSpatializationModule.GetFactory();
	SpatFactory.SetSendIPEndpoint(ADMEndpoint);

	FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
	check(AudioDeviceManager);

	AudioDeviceManager->IterateOverAllDevices([&SpatFactory, &ADMEndpoint](Audio::FDeviceId DeviceId, FAudioDevice* AudioDevice)
	{
		if (AudioDevice)
		{
			FAudioDevice::FAudioSpatializationInterfaceInfo Info = AudioDevice->GetCurrentSpatializationPluginInterfaceInfo();
			if (Info.PluginName == SpatFactory.GetDisplayName())
			{
				if (FADMSpatialization* Spatializer = static_cast<FADMSpatialization*>(AudioDevice->GetSpatializationPluginInterface().Get()))
				{
					Spatializer->SetClient(FADMClient(ADMEndpoint));
				}
			}
		}
	});
}

void UADMSpatializationSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName PropName = InPropertyChangedEvent.GetPropertyName();
	const bool bUpdatedADMClientSettings = 
		PropName == GET_MEMBER_NAME_CHECKED(UADMSpatializationSettings, IPAddress) ||
		PropName == GET_MEMBER_NAME_CHECKED(UADMSpatializationSettings, IPPort);

	if (bUpdatedADMClientSettings)
	{
		ADMConnect();
	}
}
#endif // WITH_EDITOR
