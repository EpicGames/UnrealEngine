// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include "ADMSpatializationSettings.generated.h"


UCLASS(config = ADM, defaultconfig, meta = (DisplayName = "ADM Spatialization Settings"))
class UADMSpatializationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// If set to a valid IP address, enables ADM Spatialization and sends source data to the
	// given IP Address using the OSC network protocol. Can be overridden via console variable
	// "au.OSC.ADM.SendAddress" for configuring on stand-alone client.
	UPROPERTY(config, EditAnywhere, Category = ADM, meta = (DisplayName = "OSC IP Address"))
	FString IPAddress;

	// The IP port used in conjunction with the given IP Address. Default value of 4001
	// per the ADM-OSC spec.
	UPROPERTY(config, EditAnywhere, Category = ADM, meta = (DisplayName = "OSC IP Port"))
	int32 IPPort = 4001;

#if WITH_EDITOR
	// Initializes (or reinitializes) the ADM client with the given settings.
	UFUNCTION(CallInEditor, Category = ADM, meta = (DisplayName = "Connect ADM OSC Client"))
	void ADMConnect();

	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif // WITH_EDITOR
};
